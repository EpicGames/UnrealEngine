// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BlankProgram : ModuleRules
{
	public BlankProgram(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");

		// to link with CoreUObject module:
		// PrivateDependencyModuleNames.Add("CoreUObject");

		// to enable LLM tracing:
		// GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
		// GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");
	}
}
