// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class BlankProgramTarget : TargetRules
{
	public BlankProgramTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "BlankProgram";

		// Lean and mean
		bBuildDeveloperTools = false;

		// Editor-only is enabled for desktop platforms to run unit tests that depend on editor-only data
		// It's disabled in test and shipping configs to make profiling similar to the game
		bool bDebugOrDevelopment = Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.Development;
		bBuildWithEditorOnlyData = Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop) && bDebugOrDevelopment;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bCompileICU = false;

		// to build with automation tests:
		// bForceCompileDevelopmentAutomationTests = true;

		// to enable tracing:
		// bEnableTrace = true;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
