// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlankProgram.h"

#include "Sanitizer/RaceDetector.h"
#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlankProgram, Log, All);

IMPLEMENT_APPLICATION(BlankProgram, "BlankProgram");

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::Initialize();
#endif

	FTaskTagScope Scope(ETaskTag::EGameThread);
	ON_SCOPE_EXIT
	{ 
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
#if USING_INSTRUMENTATION
		UE::Sanitizer::RaceDetector::Shutdown();
#endif
	};

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	// to run automation tests in BlankProgram:
	// * set `bForceCompileDevelopmentAutomationTests` to `true` in `BlankProgramTarget` constructor in "BlankProgram.Target.cs"
	// add `FAutomationTestFramework::Get().StartTestByName(TEXT("The name of the test class passed to IMPLEMENT_SIMPLE_AUTOMATION_TEST or TEST_CASE_NAMED"), 0);` right here

	// to link with "CoreUObject" module:
	// * uncomment `PrivateDependencyModuleNames.Add("CoreUObject");` in `BlankProgram` constructor in "BlankProgram.Build.cs"
	// * set `bCompileAgainstCoreUObject` to `true` in `BlankProgramTarget` constructor in "BlankProgram.Target.cs"

	// to enable tracing:
	// * uncomment `AppendStringToPublicDefinition("UE_TRACE_ENABLED", "1");` in `BlankProgram` constructor in "BlankProgram.Build.cs"
	// * uncomment `GlobalDefinitions.Add("UE_TRACE_ENABLED=1");` in `BlankProgramTarget` constructor in "BlankProgram.Target.cs"
	// you may need to enable compilation of a particular trace channel, e.g. for "task" traces:
	// * add `GlobalDefinitions.Add("UE_TASK_TRACE_ENABLED=1");` in `BlankProgramTarget` constructor in "BlankProgram.Target.cs"
	// you'll still need to enable this trace channel on cmd-line like `-trace=task,default`

	// to enable LLM tracing, uncomment the following in `BlankProgram` constructor in "BlankProgram.Build.cs":
	// GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
	// GlobalDefinitions.Add("UE_MEMORY_TAGS_TRACE_ENABLED=1");

	UE_LOGF(LogBlankProgram, Display, "Hello World");

	return 0;
}
